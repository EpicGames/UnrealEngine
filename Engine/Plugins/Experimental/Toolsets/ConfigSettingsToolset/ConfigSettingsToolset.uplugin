{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ConfigSettingsToolset",
	"Description": "Toolset for listing, inspecting, and editing Config Settings sections via the AI Toolset Registry.",
	"Category": "Other",
	"EditorOnly": true,
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ConfigSettingsToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": [ "Editor" ]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
