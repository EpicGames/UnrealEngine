// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConfigSettingsToolset.h"

#include "HAL/IConsoleManager.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsModule.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/ToolsetLibrary.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConfigSettingsToolset)

// --- Helpers ---
namespace UE::ConfigSettingsToolset::Private
{
	void RaiseError(const FString& Message)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("ConfigSettingsToolset: %s"), *Message));
	}

	ISettingsModule* GetSettingsModule()
	{
		return FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	}

	// Returns true if the edit can proceed. For CLASS_DefaultConfig objects, verifies the
	// backing .ini is checked out (when source control is active) or writable (otherwise),
	// raising a descriptive error and returning false when not.
	bool CheckDefaultConfigWritable(const UObject* SettingsObject)
	{
		if (!SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig))
		{
			return true;
		}

		const FString ConfigPath = FPaths::ConvertRelativePathToFull(
			SettingsObject->GetDefaultConfigFilename());

		ISourceControlModule& SCModule = ISourceControlModule::Get();
		if (SCModule.IsEnabled())
		{
			ISourceControlProvider& Provider = SCModule.GetProvider();
			FSourceControlStatePtr State = Provider.GetState(ConfigPath, EStateCacheUsage::ForceUpdate);
			if (State.IsValid() && State->IsSourceControlled() && !State->IsCheckedOut() && !State->IsAdded())
			{
				RaiseError(FString::Printf(
					TEXT("'%s' is stored in a default config file that is not checked out in source control. "
						"Check out '%s' before editing this setting."),
					*FPaths::GetCleanFilename(ConfigPath), *ConfigPath));
				return false;
			}
		}
		else if (IFileManager::Get().IsReadOnly(*ConfigPath))
		{
			RaiseError(FString::Printf(
				TEXT("'%s' is stored in a read-only default config file. "
					"Make '%s' writable before editing this setting."),
				*FPaths::GetCleanFilename(ConfigPath), *ConfigPath));
			return false;
		}

		return true;
	}

}


// --- Private ---

TSharedPtr<ISettingsSection> UConfigSettingsToolset::FindSection(
	const FString& ContainerName,
	const FString& CategoryName,
	const FString& SectionName)
{
	ISettingsModule* SettingsModule = UE::ConfigSettingsToolset::Private::GetSettingsModule();
	if (!SettingsModule)
	{
		return nullptr;
	}
	TSharedPtr<ISettingsContainer> Container = SettingsModule->GetContainer(FName(*ContainerName));
	if (!Container.IsValid())
	{
		return nullptr;
	}
	TSharedPtr<ISettingsCategory> Category = Container->GetCategory(FName(*CategoryName));
	if (!Category.IsValid())
	{
		return nullptr;
	}
	return Category->GetSection(FName(*SectionName));
}


// --- Discovery ---

TArray<FString> UConfigSettingsToolset::ListContainers()
{
	TArray<FName> Names;
	if (ISettingsModule* SettingsModule = UE::ConfigSettingsToolset::Private::GetSettingsModule())
	{
		SettingsModule->GetContainerNames(Names);
	}
	TArray<FString> Result;
	Result.Reserve(Names.Num());
	for (const FName& Name : Names)
	{
		Result.Add(Name.ToString());
	}
	Result.Sort();
	return Result;
}

TArray<FString> UConfigSettingsToolset::ListCategories(const FString& ContainerName)
{
	ISettingsModule* SettingsModule = UE::ConfigSettingsToolset::Private::GetSettingsModule();
	if (!SettingsModule)
	{
		UE::ConfigSettingsToolset::Private::RaiseError(TEXT("Settings module is not available."));
		return {};
	}
	TSharedPtr<ISettingsContainer> Container = SettingsModule->GetContainer(FName(*ContainerName));
	if (!Container.IsValid())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Container not found: %s"), *ContainerName));
		return {};
	}
	TArray<TSharedPtr<ISettingsCategory>> Categories;
	Container->GetCategories(Categories);
	TArray<FString> Result;
	Result.Reserve(Categories.Num());
	for (const TSharedPtr<ISettingsCategory>& Category : Categories)
	{
		if (Category.IsValid())
		{
			Result.Add(Category->GetName().ToString());
		}
	}
	Result.Sort();
	return Result;
}

TArray<FString> UConfigSettingsToolset::ListSections(
	const FString& ContainerName,
	const FString& CategoryName)
{
	ISettingsModule* SettingsModule = UE::ConfigSettingsToolset::Private::GetSettingsModule();
	if (!SettingsModule)
	{
		UE::ConfigSettingsToolset::Private::RaiseError(TEXT("Settings module is not available."));
		return {};
	}
	TSharedPtr<ISettingsContainer> Container = SettingsModule->GetContainer(FName(*ContainerName));
	if (!Container.IsValid())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Container not found: %s"), *ContainerName));
		return {};
	}
	TSharedPtr<ISettingsCategory> Category = Container->GetCategory(FName(*CategoryName));
	if (!Category.IsValid())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Category not found: %s/%s"), *ContainerName, *CategoryName));
		return {};
	}
	TArray<TSharedPtr<ISettingsSection>> Sections;
	Category->GetSections(Sections);
	TArray<FString> Result;
	Result.Reserve(Sections.Num());
	for (const TSharedPtr<ISettingsSection>& Section : Sections)
	{
		if (Section.IsValid())
		{
			Result.Add(Section->GetName().ToString());
		}
	}
	Result.Sort();
	return Result;
}


// --- Schema & Values ---

FString UConfigSettingsToolset::GetSectionSchema(
	const FString& ContainerName,
	const FString& CategoryName,
	const FString& SectionName)
{
	TSharedPtr<ISettingsSection> Section = FindSection(ContainerName, CategoryName, SectionName);
	if (!Section.IsValid())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section not found: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return FString();
	}
	UObject* SettingsObject = Section->GetSettingsObject().Get();
	if (!SettingsObject)
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section has no settings object (may use a custom widget): %s/%s/%s"),
				*ContainerName, *CategoryName, *SectionName));
		return FString();
	}
	return UToolsetLibrary::ListStructProperties(SettingsObject->GetClass());
}

FString UConfigSettingsToolset::GetSectionPropertyValues(
	const FString& ContainerName,
	const FString& CategoryName,
	const FString& SectionName,
	const TArray<FString>& PropertyNames)
{
	TSharedPtr<ISettingsSection> Section = FindSection(ContainerName, CategoryName, SectionName);
	if (!Section.IsValid())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section not found: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return FString();
	}
	UObject* SettingsObject = Section->GetSettingsObject().Get();
	if (!SettingsObject)
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section has no settings object (may use a custom widget): %s/%s/%s"),
				*ContainerName, *CategoryName, *SectionName));
		return FString();
	}
	TArray<FName> Names;
	Names.Reserve(PropertyNames.Num());
	for (const FString& Name : PropertyNames)
	{
		Names.Add(FName(*Name));
	}
	return UToolsetLibrary::GetObjectProperties(SettingsObject, Names);
}


// --- Editing ---

bool UConfigSettingsToolset::SetSectionProperties(
	const FString& ContainerName,
	const FString& CategoryName,
	const FString& SectionName,
	const FString& PropertiesJson)
{
	TSharedPtr<ISettingsSection> Section = FindSection(ContainerName, CategoryName, SectionName);
	if (!Section.IsValid())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section not found: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return false;
	}
	if (!Section->CanEdit())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section cannot be edited: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return false;
	}
	UObject* SettingsObject = Section->GetSettingsObject().Get();
	if (!SettingsObject)
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section has no settings object: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return false;
	}
	if (!UE::ConfigSettingsToolset::Private::CheckDefaultConfigWritable(SettingsObject))
	{
		return false;
	}
	TArray<FName> SetPropertyNames;
	const bool bSetSuccess = UToolsetLibrary::SetObjectProperties(SettingsObject, PropertiesJson, SetPropertyNames, EBypassContainerCheck::No);
	if(SetPropertyNames.IsEmpty())
	{
		// Only return here if no properties were set successfully.
		// Otherwise continue to save the file since these are not assets, so UE will not prompt us to save
		// the changes later.
		return bSetSuccess;
	}

	// Per-property config save (mirrors SSettingsEditor::HandleSettingsPropertyChanged).
	// Use UpdateSinglePropertyInConfigFile for individual DefaultConfig properties;
	// fall back to a full Section->Save() for collection types when the CVar is absent/0.
	const UClass* ObjectClass = SettingsObject->GetClass();
	const bool bIsDefaultConfig = ObjectClass->HasAnyClassFlags(CLASS_DefaultConfig);
	if(!bIsDefaultConfig)
	{
		if(!Section->Save())
		{
			UE::ConfigSettingsToolset::Private::RaiseError(
				FString::Printf(TEXT("Failed to save section: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
			return false;			
		}
		return bSetSuccess;
	}

	if(!Section->NotifySectionOnPropertyModified())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Failed to notify section property modified: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return false;
	}

	IConsoleVariable* NewPropertySavingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("ini.UseNewPropertySaving"));
	const bool bCanUsePerPropertySaveForContainers = NewPropertySavingCVar && NewPropertySavingCVar->GetInt() != 0;

	for (const FName& PropertyName : SetPropertyNames)
	{
		FProperty* Prop = FindFProperty<FProperty>(ObjectClass, PropertyName);
		if(ensureAlways(Prop))
		{
			if(bCanUsePerPropertySaveForContainers || 
				!(Prop->IsA<FArrayProperty>() || Prop->IsA<FSetProperty>() || Prop->IsA<FMapProperty>()))
			{
				SettingsObject->UpdateSinglePropertyInConfigFile(Prop, SettingsObject->GetDefaultConfigFilename());
			}
			else
			{
				// Once the whole section has been saved, we no longer need to iterate individual properties
				if (!Section->Save())
				{
					UE::ConfigSettingsToolset::Private::RaiseError(
						FString::Printf(TEXT("Failed to save section: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
					return false;
				}
				return bSetSuccess;
			}
		}
	}
	return bSetSuccess;
}

bool UConfigSettingsToolset::SaveSection(
	const FString& ContainerName,
	const FString& CategoryName,
	const FString& SectionName)
{
	TSharedPtr<ISettingsSection> Section = FindSection(ContainerName, CategoryName, SectionName);
	if (!Section.IsValid())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section not found: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return false;
	}
	if (!Section->CanSave())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section does not support saving: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return false;
	}
	return Section->Save();
}

bool UConfigSettingsToolset::ResetSectionToDefaults(
	const FString& ContainerName,
	const FString& CategoryName,
	const FString& SectionName)
{
	TSharedPtr<ISettingsSection> Section = FindSection(ContainerName, CategoryName, SectionName);
	if (!Section.IsValid())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section not found: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return false;
	}
	if (!Section->CanResetDefaults())
	{
		UE::ConfigSettingsToolset::Private::RaiseError(
			FString::Printf(TEXT("Section does not support reset to defaults: %s/%s/%s"), *ContainerName, *CategoryName, *SectionName));
		return false;
	}
	return Section->ResetDefaults();
}
