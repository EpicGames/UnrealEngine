// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISettingsSection.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

#include "ConfigSettingsToolset.generated.h"

/// Tools for listing, inspecting, and editing Config Settings sections
UCLASS(BlueprintType, MinimalAPI)
class UConfigSettingsToolset : public UToolsetDefinition
{
	GENERATED_BODY()
public:

	// --- Discovery ---

	/**
	 * Lists the names of all known settings containers, sorted alphabetically.
	 * Common containers are "Editor" and "Project".
	 * @return Sorted array of container names.
	 */
	UFUNCTION(meta = (AICallable), Category = "ConfigSettingsToolset")
	static TArray<FString> ListContainers();

	/**
	 * Lists the names of all categories within a settings container, sorted alphabetically.
	 * Raises an error if the container does not exist.
	 * @param ContainerName The name of the container (e.g. "Project").
	 * @return Sorted array of category names.
	 */
	UFUNCTION(meta = (AICallable), Category = "ConfigSettingsToolset")
	static TArray<FString> ListCategories(const FString& ContainerName);

	/**
	 * Lists the names of all sections within a settings category, sorted alphabetically.
	 * Raises an error if the container or category does not exist.
	 * @param ContainerName The name of the container (e.g. "Project").
	 * @param CategoryName The name of the category (e.g. "Engine").
	 * @return Sorted array of section names.
	 */
	UFUNCTION(meta = (AICallable), Category = "ConfigSettingsToolset")
	static TArray<FString> ListSections(const FString& ContainerName, const FString& CategoryName);

	// --- Schema & Values ---

	/**
	 * Returns a JSON Schema describing the user-visible properties of a settings section.
	 * The schema maps each property name to its type, description, and constraints.
	 * Raises an error if the section does not exist or has no backing settings object
	 * (e.g. uses a custom widget instead).
	 * @param ContainerName The name of the container (e.g. "Project").
	 * @param CategoryName The name of the category (e.g. "Engine").
	 * @param SectionName The name of the section (e.g. "General").
	 * @return JSON Schema string describing the section's properties, or empty string on failure.
	 */
	UFUNCTION(meta = (AICallable), Category = "ConfigSettingsToolset")
	static FString GetSectionSchema(
		const FString& ContainerName,
		const FString& CategoryName,
		const FString& SectionName);

	/**
	 * Returns the current values of the specified properties as a JSON object.
	 * Raises an error if the section does not exist, has no settings object, or any
	 * requested property cannot be read.
	 * @param ContainerName The name of the container (e.g. "Project").
	 * @param CategoryName The name of the category (e.g. "Engine").
	 * @param SectionName The name of the section (e.g. "General").
	 * @param PropertyNames The names of the properties to read.
	 * @return JSON object mapping property names to their current values, or empty string on failure.
	 */
	UFUNCTION(meta = (AICallable), Category = "ConfigSettingsToolset")
	static FString GetSectionPropertyValues(
		const FString& ContainerName,
		const FString& CategoryName,
		const FString& SectionName,
		const TArray<FString>& PropertyNames);

	// --- Editing ---

	/**
	 * Sets one or more properties on a settings section from a JSON object and saves.
	 * PropertiesJson must be a JSON object mapping property names to new values,
	 * in the same format returned by GetSectionPropertyValues.
	 * Raises an error if the section does not exist, cannot be edited, has no settings
	 * object, the default config file is not writable, or any property cannot be set.
	 * @param ContainerName The name of the container (e.g. "Project").
	 * @param CategoryName The name of the category (e.g. "Engine").
	 * @param SectionName The name of the section (e.g. "General").
	 * @param PropertiesJson JSON object with property name to new value pairs.
	 * @return True if all properties successfully set.
	 */
	UFUNCTION(meta = (AICallable), Category = "ConfigSettingsToolset")
	static bool SetSectionProperties(
		const FString& ContainerName,
		const FString& CategoryName,
		const FString& SectionName,
		const FString& PropertiesJson);

	/**
	 * Saves the settings in a section.
	 * Raises an error if the section does not exist or saving is not supported.
	 * @param ContainerName The name of the container (e.g. "Project").
	 * @param CategoryName The name of the category (e.g. "Engine").
	 * @param SectionName The name of the section (e.g. "General").
	 * @return True on success.
	 */
	UFUNCTION(meta = (AICallable), Category = "ConfigSettingsToolset")
	static bool SaveSection(
		const FString& ContainerName,
		const FString& CategoryName,
		const FString& SectionName);

	/**
	 * Resets the settings in a section to their default values.
	 * Raises an error if the section does not exist or reset is not supported.
	 * @param ContainerName The name of the container (e.g. "Project").
	 * @param CategoryName The name of the category (e.g. "Engine").
	 * @param SectionName The name of the section (e.g. "General").
	 * @return True on success.
	 */
	UFUNCTION(meta = (AICallable), Category = "ConfigSettingsToolset")
	static bool ResetSectionToDefaults(
		const FString& ContainerName,
		const FString& CategoryName,
		const FString& SectionName);

private:
	friend class FConfigSettingsToolsetSpec;

	static TSharedPtr<ISettingsSection> FindSection(
		const FString& ContainerName,
		const FString& CategoryName,
		const FString& SectionName);
};
