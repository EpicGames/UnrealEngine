// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"
#include "ConfigSettingsToolset.h"

#define LOCTEXT_NAMESPACE "FConfigSettingsToolsetModule"

class FConfigSettingsToolsetModule : public IModuleInterface
{
	void StartupModule()
	{
		UToolsetRegistry::RegisterToolsetClass(UConfigSettingsToolset::StaticClass());
	}

	void ShutdownModule()
	{
		UToolsetRegistry::UnregisterToolsetClass(UConfigSettingsToolset::StaticClass());
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FConfigSettingsToolsetModule, ConfigSettingsToolset)
