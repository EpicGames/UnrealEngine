// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "Misc/AutomationTest.h"
#include "Misc/ScopeExit.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "SourceControlHelpers.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"

#include "ConfigSettingsToolset.h"
#include "Tests/ConfigSettingsToolsetTestObjects.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace
{
const EAutomationTestFlags TestFlags =
	EAutomationTestFlags::EditorContext |
	EAutomationTestFlags::ProductFilter |
	EAutomationTestFlags::CriticalPriority;

// Known-stable test coordinates: always registered by ProjectSettingsViewerModule.
const FString TestContainer = TEXT("Project");
const FString TestCategory  = TEXT("Project");
const FString TestSection   = TEXT("General");
}

BEGIN_DEFINE_SPEC(
	FConfigSettingsToolsetSpec,
	TEXT("AI.Toolsets.ConfigSettingsToolset"),
	TestFlags)

// Searches all registered sections for one matching Predicate.
// Returns false if none found. Uses ISettingsModule directly to avoid
// going through the toolset exception handler during discovery.
bool FindSectionWhere(
	TFunctionRef<bool(ISettingsSection&)> Predicate,
	FString& OutContainer, FString& OutCategory, FString& OutSection)
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (!SettingsModule)
	{
		return false;
	}
	TArray<FName> ContainerNames;
	SettingsModule->GetContainerNames(ContainerNames);
	for (const FName& ContainerName : ContainerNames)
	{
		TSharedPtr<ISettingsContainer> Container = SettingsModule->GetContainer(ContainerName);
		if (!Container.IsValid())
		{
			continue;
		}
		TArray<TSharedPtr<ISettingsCategory>> Categories;
		Container->GetCategories(Categories);
		for (const TSharedPtr<ISettingsCategory>& Category : Categories)
		{
			if (!Category.IsValid())
			{
				continue;
			}
			TArray<TSharedPtr<ISettingsSection>> Sections;
			Category->GetSections(Sections);
			for (const TSharedPtr<ISettingsSection>& Section : Sections)
			{
				if (Section.IsValid() && Predicate(*Section))
				{
					OutContainer = ContainerName.ToString();
					OutCategory  = Category->GetName().ToString();
					OutSection   = Section->GetName().ToString();
					return true;
				}
			}
		}
	}
	return false;
}

void ExpectNoException()
{
	check(ExceptionHandler.IsValid());
	TestEqual(TEXT("Error"), *ExceptionHandler->GetException(), TEXT(""));
	ExceptionHandler.Reset();
}

void ExpectExceptionContains(const FString& ErrorMessage)
{
	check(ExceptionHandler.IsValid());
	TestTrue(*FString::Printf(TEXT("Error contains '%s'"), *ErrorMessage),
		ExceptionHandler->GetException().Contains(ErrorMessage));
	ExceptionHandler.Reset();
}

private:
	TUniquePtr<UE::ToolsetRegistry::FToolCallExceptionHandler> ExceptionHandler;
END_DEFINE_SPEC(FConfigSettingsToolsetSpec)

void FConfigSettingsToolsetSpec::Define()
{
	BeforeEach([this]()
	{
		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
	});

	AfterEach([this]()
	{
		if (ExceptionHandler.IsValid())
		{
			ExpectNoException();
		}
	});


	Describe(TEXT("ListContainers"), [this]()
	{
		It(TEXT("returns a sorted list"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UConfigSettingsToolset::ListContainers();
				for (int32 i = 1; i < Names.Num(); ++i)
				{
					TestTrue(TEXT("Sorted"), Names[i - 1] <= Names[i]);
				}
			});
			ExpectNoException();
		});

		It(TEXT("contains 'Project'"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UConfigSettingsToolset::ListContainers();
				TestTrue(TEXT("Contains Project"), Names.Contains(TestContainer));
			});
			ExpectNoException();
		});
	});

	Describe(TEXT("ListCategories"), [this]()
	{
		It(TEXT("returns categories for 'Project'"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UConfigSettingsToolset::ListCategories(TestContainer);
				TestFalse(TEXT("Non-empty"), Names.IsEmpty());
			});
			ExpectNoException();
		});

		It(TEXT("raises error for nonexistent container"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::ListCategories(TEXT("NonexistentContainerXYZ12345"));
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});
	});

	Describe(TEXT("ListSections"), [this]()
	{
		It(TEXT("returns sections for 'Project'/'Project'"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const TArray<FString> Names = UConfigSettingsToolset::ListSections(TestContainer, TestCategory);
				TestFalse(TEXT("Non-empty"), Names.IsEmpty());
			});
			ExpectNoException();
		});

		It(TEXT("raises error for nonexistent category"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::ListSections(TestContainer, TEXT("NonexistentCategoryXYZ12345"));
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});
	});

	Describe(TEXT("GetSectionSchema"), [this]()
	{
		It(TEXT("returns schema for 'Project'/'Project'/'General'"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const FString Schema =
					UConfigSettingsToolset::GetSectionSchema(TestContainer, TestCategory, TestSection);
				TestFalse(TEXT("Non-empty"), Schema.IsEmpty());
			});
			ExpectNoException();
		});

		It(TEXT("raises error for nonexistent section"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::GetSectionSchema(
					TestContainer, TestCategory, TEXT("NonexistentSectionXYZ12345"));
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});
	});

	Describe(TEXT("GetSectionPropertyValues"), [this]()
	{
		It(TEXT("returns values for 'Project'/'Project'/'General'"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const FString Values = UConfigSettingsToolset::GetSectionPropertyValues(
					TestContainer, TestCategory, TestSection,
					{TEXT("bShouldWindowPreserveAspectRatio")});
				TestFalse(TEXT("Non-empty"), Values.IsEmpty());
			});
			ExpectNoException();
		});

		It(TEXT("raises error for nonexistent section"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::GetSectionPropertyValues(
					TestContainer, TestCategory, TEXT("NonexistentSectionXYZ12345"),
					{TEXT("bShouldWindowPreserveAspectRatio")});
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});
	});

	Describe(TEXT("SetSectionProperties"), [this]()
	{
		It(TEXT("raises read-only error for 'Project'/'Project'/'General' when default config file is not writable"), [this]()
		{
			// This test verifies the error path when the backing DefaultConfig .ini is
			// physically read-only and source control is not active (e.g. Horde build agents).
			// It skips when the file is writable or SC is active, since those paths are
			// covered by the separate writable-section round-trip test.
			ISettingsModule* SM = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
			TSharedPtr<ISettingsContainer> Container = SM ? SM->GetContainer(FName(*TestContainer)) : nullptr;
			TSharedPtr<ISettingsCategory> Category = Container.IsValid() ? Container->GetCategory(FName(*TestCategory)) : nullptr;
			TSharedPtr<ISettingsSection> Section = Category.IsValid() ? Category->GetSection(FName(*TestSection)) : nullptr;
			UObject* SettingsObject = Section.IsValid() ? Section->GetSettingsObject().Get() : nullptr;

			if (!SettingsObject || !SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig))
			{
				AddInfo(TEXT("Expected DefaultConfig settings object not found; skipping."));
				ExceptionHandler.Reset();
				return;
			}

			const FString ConfigPath = FPaths::ConvertRelativePathToFull(SettingsObject->GetDefaultConfigFilename());

			if (ISourceControlModule::Get().IsEnabled() || !IFileManager::Get().IsReadOnly(*ConfigPath))
			{
				AddInfo(TEXT("Default config file is writable or source control is active; read-only error case not exercisable here."));
				ExceptionHandler.Reset();
				return;
			}

			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::SetSectionProperties(
					TestContainer, TestCategory, TestSection,
					TEXT("{\"bShouldWindowPreserveAspectRatio\": true}"));
			});
			ExpectExceptionContains(TEXT("read-only default config file"));
		});

		It(TEXT("raises check-out error for 'Project'/'Project'/'General' when source control is active and file is not checked out"), [this]()
		{
			// Verifies the SC-specific error path: SC active but DefaultConfig .ini not checked out.
			// Skips when SC is inactive (covered by the read-only test) or file is already checked out.
			ISettingsModule* SM = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
			TSharedPtr<ISettingsContainer> Container = SM ? SM->GetContainer(FName(*TestContainer)) : nullptr;
			TSharedPtr<ISettingsCategory> Category = Container.IsValid() ? Container->GetCategory(FName(*TestCategory)) : nullptr;
			TSharedPtr<ISettingsSection> Section = Category.IsValid() ? Category->GetSection(FName(*TestSection)) : nullptr;
			UObject* SettingsObject = Section.IsValid() ? Section->GetSettingsObject().Get() : nullptr;

			if (!SettingsObject || !SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig))
			{
				AddInfo(TEXT("Expected DefaultConfig settings object not found; skipping."));
				ExceptionHandler.Reset();
				return;
			}

			ISourceControlModule& SCModule = ISourceControlModule::Get();
			if (!SCModule.IsEnabled())
			{
				AddInfo(TEXT("Source control is not active; skipping."));
				ExceptionHandler.Reset();
				return;
			}

			const FString ConfigPath = FPaths::ConvertRelativePathToFull(SettingsObject->GetDefaultConfigFilename());
			FSourceControlStatePtr State = SCModule.GetProvider().GetState(ConfigPath, EStateCacheUsage::ForceUpdate);
			if (!State.IsValid() || !State->IsSourceControlled())
			{
				AddInfo(TEXT("Default config file is not source controlled; skipping."));
				ExceptionHandler.Reset();
				return;
			}

			if (State->IsCheckedOut() || State->IsAdded())
			{
				AddInfo(TEXT("Default config file is already checked out; skipping."));
				ExceptionHandler.Reset();
				return;
			}

			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::SetSectionProperties(
					TestContainer, TestCategory, TestSection,
					TEXT("{\"bShouldWindowPreserveAspectRatio\": true}"));
			});
			ExpectExceptionContains(TEXT("not checked out in source control"));
		});

		It(TEXT("round-trips a property on a registered writable section"), [this]()
		{
			// Uses a custom non-DefaultConfig settings object so the write path always
			// succeeds regardless of source control state or file permissions.
			ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
			if (!SettingsModule)
			{
				AddInfo(TEXT("Settings module not available; skipping."));
				ExceptionHandler.Reset();
				return;
			}

			const FString WritableContainer = TEXT("ConfigSettingsToolsetTest");
			const FString WritableCategory  = TEXT("Test");
			const FString WritableSection   = TEXT("Properties");

			UConfigSettingsToolsetTestSettings* TestObj = NewObject<UConfigSettingsToolsetTestSettings>();
			TestObj->AddToRoot();

			SettingsModule->RegisterSettings(
				FName(*WritableContainer), FName(*WritableCategory), FName(*WritableSection),
				INVTEXT("Test Section"), INVTEXT("Automation test settings"),
				TestObj);

			ON_SCOPE_EXIT
			{
				SettingsModule->UnregisterSettings(
					FName(*WritableContainer), FName(*WritableCategory), FName(*WritableSection));
				TestObj->RemoveFromRoot();
			};

			ExceptionHandler->CaptureErrorsIn([this, &WritableContainer, &WritableCategory, &WritableSection]()
			{
				const FString Original = UConfigSettingsToolset::GetSectionPropertyValues(
					WritableContainer, WritableCategory, WritableSection,
					{TEXT("bTestBoolProperty")});

				const bool bSet = UConfigSettingsToolset::SetSectionProperties(
					WritableContainer, WritableCategory, WritableSection, Original);
				TestTrue(TEXT("Set succeeded"), bSet);

				const FString After = UConfigSettingsToolset::GetSectionPropertyValues(
					WritableContainer, WritableCategory, WritableSection,
					{TEXT("bTestBoolProperty")});
				TestEqual(TEXT("Value unchanged after round-trip"), After, Original);
			});
			ExpectNoException();
		});

		It(TEXT("raises error for nonexistent section"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::SetSectionProperties(
					TestContainer, TestCategory, TEXT("NonexistentSectionXYZ12345"),
					TEXT("{\"bShouldWindowPreserveAspectRatio\": true}"));
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});

		It(TEXT("raises error for nonexistent property"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::SetSectionProperties(
					TestContainer, TestCategory, TestSection,
					TEXT("{\"NonexistentPropertyXYZ12345\": true}"));
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});

		It(TEXT("raises error for unparseable value"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::SetSectionProperties(
					TestContainer, TestCategory, TestSection,
					TEXT("{\"bShouldWindowPreserveAspectRatio\": \"NOT_A_VALID_BOOL_XYZ\"}"));
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});
	});

	Describe(TEXT("SaveSection"), [this]()
	{
		It(TEXT("saves 'Project'/'Project'/'General' without error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				const bool bSaved = UConfigSettingsToolset::SaveSection(TestContainer, TestCategory, TestSection);
				TestTrue(TEXT("Saved"), bSaved);
			});
			ExpectNoException();
		});

		It(TEXT("raises error when section does not support saving"), [this]()
		{
			FString Container, Category, Section;
			if (!FindSectionWhere([](const ISettingsSection& S){ return !S.CanSave(); },
				Container, Category, Section))
			{
				AddInfo(TEXT("No section with CanSave=false found; skipping assertion."));
				ExceptionHandler.Reset();
				return;
			}
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				UConfigSettingsToolset::SaveSection(Container, Category, Section);
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});

		It(TEXT("raises error for nonexistent section"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::SaveSection(
					TestContainer, TestCategory, TEXT("NonexistentSectionXYZ12345"));
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});
	});

	Describe(TEXT("ResetSectionToDefaults"), [this]()
	{
		It(TEXT("resets a resettable section to defaults without error"), [this]()
		{
			FString Container, Category, Section;
			if (!FindSectionWhere([](ISettingsSection& S) 
			{ 
				// Don't run this test on the sections with OnResetDefaults bound 
				// because it might expect interaction
				return S.CanResetDefaults() && !S.OnResetDefaults().IsBound(); 
			},
				Container, Category, Section))
			{
				AddInfo(TEXT("No section with CanResetDefaults=true found; skipping assertion."));
				ExceptionHandler.Reset();
				return;
			}
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				const bool bReset = UConfigSettingsToolset::ResetSectionToDefaults(Container, Category, Section);
				TestTrue(TEXT("Reset"), bReset);
			});
			ExpectNoException();
		});

		It(TEXT("raises error when section does not support reset"), [this]()
		{
			FString Container, Category, Section;
			if (!FindSectionWhere([](const ISettingsSection& S){ return !S.CanResetDefaults(); },
				Container, Category, Section))
			{
				AddInfo(TEXT("No section with CanResetDefaults=false found; skipping assertion."));
				ExceptionHandler.Reset();
				return;
			}
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				UConfigSettingsToolset::ResetSectionToDefaults(Container, Category, Section);
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});

		It(TEXT("raises error for nonexistent section"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
			{
				UConfigSettingsToolset::ResetSectionToDefaults(
					TestContainer, TestCategory, TEXT("NonexistentSectionXYZ12345"));
			});
			ExpectExceptionContains(TEXT("ConfigSettingsToolset"));
		});
	});
}
#endif
