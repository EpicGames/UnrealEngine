// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "ConfigSettingsToolsetTestObjects.generated.h"

/**
 * Settings object used exclusively by ConfigSettingsToolsetTest.
 * Uses config=ConfigSettingsToolsetTest (NOT DefaultConfig) so SaveConfig()
 * writes to Saved/Config/ — a directory that is writable on all machines
 * including Horde build agents.
 */
UCLASS(config=ConfigSettingsToolsetTest)
class UConfigSettingsToolsetTestSettings : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(config, EditAnywhere, Category = "Test")
	bool bTestBoolProperty = false;
};
